// Copyright Epic Games, Inc. All Rights Reserved.

#include "AMF.h"

#include "HAL/PlatformProcess.h"

REGISTER_TYPEID(FAMF);

FAMF::FAMF()
{
#if PLATFORM_WINDOWS
	// Early out on non-supported windows versions
	if (!FPlatformMisc::VerifyWindowsVersion(6, 2))
	{
		return;
	}
#endif

#ifdef AMF_DLL_NAMEA
	// To avoid a warning during tests we manually call dlopen on Linux as this is how we currently determine if AMF is avaliable
	#if PLATFORM_LINUX && PLATFORM_DESKTOP
	DllHandle = dlopen( AMF_DLL_NAMEA, RTLD_LAZY | RTLD_NOLOAD | RTLD_GLOBAL );
#else 
	DllHandle = FPlatformProcess::GetDllHandle(TEXT(AMF_DLL_NAMEA));
#endif
#endif

	if (DllHandle)
	{
		AMFInit_Fn AmfInitFn = (AMFInit_Fn)FPlatformProcess::GetDllExport(DllHandle, TEXT(AMF_INIT_FUNCTION_NAME));

		if (AmfInitFn != nullptr)
		{
			AMF_RESULT const Result = AmfInitFn(AMF_FULL_VERSION, &Factory);
			if (Result == AMF_OK || Result == AMF_ALREADY_INITIALIZED)
			{
				return;
			}
		}

		DllHandle = nullptr;
	}
}

bool FAMF::IsValid() const
{
	return DllHandle != nullptr && bHasCompatibleGPU;
}
