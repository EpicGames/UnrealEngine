// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"

#include "Video/Decoders/Configs/VideoDecoderConfigH264.h"
#include "Video/Decoders/Configs/VideoDecoderConfigH265.h"
#include "Video/Encoders/Configs/VideoEncoderConfigH264.h"
#include "Video/Encoders/Configs/VideoEncoderConfigH265.h"

#include "Video/Resources/Vulkan/VideoResourceVulkan.h"

#if PLATFORM_WINDOWS
#include "Video/Resources/D3D/VideoResourceD3D.h"
#endif

#include "Video/Decoders/VideoDecoderAMF.h"
#include "Video/Encoders/VideoEncoderAMF.h"

class FAMFCodecModule : public IModuleInterface
{
public:
	virtual void StartupModule() override
	{
#if PLATFORM_WINDOWS
		// TODO (Eden.Harris) H265/HEVC disabled because of intermittent build failure. See UE-239424
		FVideoEncoder::RegisterPermutationsOf<TVideoEncoderAMF<FVideoResourceD3D11>>
			::With<FVideoResourceD3D11>
			::And<FVideoEncoderConfigAMF, FVideoEncoderConfigH264>([](TSharedRef<FAVDevice> const& NewDevice, TSharedRef<FAVInstance> const& NewInstance)
				{
					return FAPI::Get<FAMF>().IsValid();
				});

		FVideoEncoder::RegisterPermutationsOf<TVideoEncoderAMF<FVideoResourceD3D12>>
			::With<FVideoResourceD3D12>
			::And<FVideoEncoderConfigAMF, FVideoEncoderConfigH264>([](TSharedRef<FAVDevice> const& NewDevice, TSharedRef<FAVInstance> const& NewInstance)
				{
					return FAPI::Get<FAMF>().IsValid();
				});

		// TODO (william.belcher): H265 decoding doesn't currently work with AMD
		/* 
		FVideoDecoder::RegisterPermutationsOf<TVideoDecoderAMF<FVideoResourceD3D11>>
			::With<FVideoResourceD3D11>
			::And<FVideoDecoderConfigAMF, FVideoDecoderConfigH264, FVideoDecoderConfigH265>([](TSharedRef<FAVDevice> const& NewDevice, TSharedRef<FAVInstance> const& NewInstance)
				{
					return FAPI::Get<FAMF>().IsValid();
				});

		FVideoDecoder::RegisterPermutationsOf<TVideoDecoderAMF<FVideoResourceD3D12>>
			::With<FVideoResourceD3D12>
			::And<FVideoDecoderConfigAMF, FVideoDecoderConfigH264, FVideoDecoderConfigH265>([](TSharedRef<FAVDevice> const& NewDevice, TSharedRef<FAVInstance> const& NewInstance)
				{
					return FAPI::Get<FAMF>().IsValid();
				});
		*/
#endif

		/*FVideoEncoder::RegisterPermutationsOf<TVideoEncoderAMF<FVideoResourceVulkan>>
			::With<FVideoResourceVulkan>
			::And<FVideoEncoderConfigAMF, FVideoEncoderConfigH264, FVideoEncoderConfigH265>([](TSharedRef<FAVDevice> const& NewDevice, TSharedRef<FAVInstance> const& NewInstance)
				{
					return FAPI::Get<FAMF>().IsValid();
				});

		FVideoDecoder::RegisterPermutationsOf<TVideoDecoderAMF<FVideoResourceVulkan>>
			::With<FVideoResourceVulkan>
			::And<FVideoDecoderConfigAMF, FVideoDecoderConfigH264, FVideoDecoderConfigH265>([](TSharedRef<FAVDevice> const& NewDevice, TSharedRef<FAVInstance> const& NewInstance)
				{
					return FAPI::Get<FAMF>().IsValid();
				});*/
	}
};

IMPLEMENT_MODULE(FAMFCodecModule, AMFCodecs);
