// Copyright Epic Games, Inc. All Rights Reserved.

#include "Video/Decoders/Configs/VideoDecoderConfigAMF.h"

#include "components/VideoDecoderUVD.h"

#include "Video/Decoders/Configs/VideoDecoderConfigH264.h"
#include "Video/Decoders/Configs/VideoDecoderConfigH265.h"

REGISTER_TYPEID(FVideoDecoderConfigAMF);

FName const FVideoDecoderConfigAMF::CodecTypeH264 = WCHAR_TO_TCHAR(AMFVideoDecoderUVD_H264_AVC);
FName const FVideoDecoderConfigAMF::CodecTypeH265 = WCHAR_TO_TCHAR(AMFVideoDecoderHW_H265_HEVC);

FAVResult FVideoDecoderConfigAMF::Parse(TSharedRef<FAVInstance> const& Instance, FVideoPacket const& Packet, TArray<FParsedPicture>& OutPictures)
{
	if (CodecType == CodecTypeH264)
	{
		
	}
	else if (CodecType == CodecTypeH265)
	{
		using namespace UE::AVCodecCore::H265;

		FVideoDecoderConfigH265& H265 = Instance->Edit<FVideoDecoderConfigH265>();

		// TArray<TSharedPtr<FNaluH265>> Nalus;
		// FAVResult const Result = H265.Parse(Packet, Nalus);
		// if (Result.IsNotSuccess())
		// {
		// 	return FAVResult(EAVResult::ErrorResolving, TEXT("Failed to parse H265 Bitstream"));
		// }

		// TODO (Aidan) Write SPS+PPS in contiguous block to each FParsedPicture in OutPictures

		return EAVResult::Success;
	}

	return FAVResult(EAVResult::ErrorMapping, FString::Printf(TEXT("Unsupported codec type %s"), *CodecType.ToString()), TEXT("AMF"));
}

template<>
DLLEXPORT FAVResult FAVExtension::TransformConfig(FVideoDecoderConfigAMF& OutConfig, FVideoDecoderConfig const& InConfig)
{
	OutConfig.Preset = InConfig.Preset;
	OutConfig.LatencyMode = InConfig.LatencyMode;

	return EAVResult::Success;
}

template <>
DLLEXPORT FAVResult FAVExtension::TransformConfig(FVideoDecoderConfig& OutConfig, FVideoDecoderConfigAMF const& InConfig)
{
	OutConfig.Preset = InConfig.Preset;
	OutConfig.LatencyMode = InConfig.LatencyMode;

	return EAVResult::Success;
}

template <>
DLLEXPORT FAVResult FAVExtension::TransformConfig(FVideoDecoderConfigAMF& OutConfig, FVideoDecoderConfigH264 const& InConfig)
{
	OutConfig.CodecType = FVideoDecoderConfigAMF::CodecTypeH264;

	return FAVExtension::TransformConfig<FVideoDecoderConfigAMF, FVideoDecoderConfig>(OutConfig, InConfig);
}

template <>
DLLEXPORT FAVResult FAVExtension::TransformConfig(FVideoDecoderConfigAMF& OutConfig, FVideoDecoderConfigH265 const& InConfig)
{
	OutConfig.CodecType = FVideoDecoderConfigAMF::CodecTypeH265;

	return FAVExtension::TransformConfig<FVideoDecoderConfigAMF, FVideoDecoderConfig>(OutConfig, InConfig);
}
