// Copyright Epic Games, Inc. All Rights Reserved.

#include "Video/Decoders/VideoDecoderAMF.h"

#include "Video/Resources/Vulkan/VideoResourceVulkan.h"

#if PLATFORM_WINDOWS
#include "Video/Resources/D3D/VideoResourceD3D.h"
#endif

#if PLATFORM_WINDOWS

template <>
DLLEXPORT FAVResult FVideoDecoderAMF::SetupContext<FVideoResourceD3D11>(TVideoDecoderAMF<FVideoResourceD3D11>& This)
{
	TSharedPtr<FVideoContextD3D11> const& D3D11Context = This.GetDevice()->GetContext<FVideoContextD3D11>();
	if (D3D11Context.IsValid())
	{
		AMF_RESULT const Result = This.Context->InitDX11(D3D11Context->Device);
		if (Result != AMF_OK)
		{
			return FAVResult(EAVResult::ErrorCreating, TEXT("Failed to initialize the D3D11 decoder context"), TEXT("AMF"), Result);
		}
	}

	return EAVResult::Success;
}

template <>
DLLEXPORT FAVResult FVideoDecoderAMF::CopySurface<FVideoResourceD3D11>(TVideoDecoderAMF<FVideoResourceD3D11>& This, TSharedPtr<FVideoResourceD3D11>& OutResource, amf::AMFSurfacePtr const& InSurface)
{
	if (OutResource->GetDevice()->HasContext<FVideoContextD3D11>())
	{
		ID3D11DeviceContext* DeviceContext = nullptr;
		OutResource->GetDevice()->GetContext<FVideoContextD3D11>()->Device->GetImmediateContext(&DeviceContext);

		DeviceContext->CopySubresourceRegion(OutResource->GetRaw(), 0, 0, 0, 0, static_cast<ID3D11Resource*>(InSurface->GetPlaneAt(0)->GetNative()), 0, nullptr);
		DeviceContext->Flush();

		DeviceContext->Release();

		return EAVResult::Success;
	}

	return FAVResult(EAVResult::ErrorMapping, TEXT("No D3D11 context found"), TEXT("AMF"));
}

template <>
DLLEXPORT FAVResult FVideoDecoderAMF::SetupContext<FVideoResourceD3D12>(TVideoDecoderAMF<FVideoResourceD3D12>& This)
{
	TSharedPtr<FVideoContextD3D12> const& D3D12Context = This.GetDevice()->GetContext<FVideoContextD3D12>();
	if (D3D12Context.IsValid())
	{
		AMF_RESULT const Result = amf::AMFContext2Ptr(This.Context)->InitDX12(D3D12Context->Device);
		if (Result != AMF_OK)
		{
			return FAVResult(EAVResult::ErrorCreating, TEXT("Failed to initialize the D3D12 decoder context"), TEXT("AMF"), Result);
		}
	}

	return EAVResult::Success;
}

template <>
DLLEXPORT FAVResult FVideoDecoderAMF::CopySurface<FVideoResourceD3D12>(TVideoDecoderAMF<FVideoResourceD3D12>& This, TSharedPtr<FVideoResourceD3D12>& OutResource, amf::AMFSurfacePtr const& InSurface)
{
	return EAVResult::FatalUnsupported;
}

#endif

template <>
DLLEXPORT FAVResult FVideoDecoderAMF::SetupContext<FVideoResourceVulkan>(TVideoDecoderAMF<FVideoResourceVulkan>& This)
{
	TSharedPtr<FVideoContextVulkan> const& VulkanContext = This.GetDevice()->GetContext<FVideoContextVulkan>();
	if (VulkanContext.IsValid())
	{
		amf::AMFVulkanDevice AMFDevice = {};
		AMFDevice.cbSizeof = sizeof(amf::AMFVulkanDevice);
		AMFDevice.hInstance = VulkanContext->Instance;
		AMFDevice.hPhysicalDevice = VulkanContext->PhysicalDevice;
		AMFDevice.hDevice = VulkanContext->Device;

		AMF_RESULT const Result = amf::AMFContext1Ptr(This.Context)->InitVulkan(&AMFDevice);
		if (Result != AMF_OK)
		{
			return FAVResult(EAVResult::ErrorCreating, TEXT("Failed to initialize the vulkan decoder context"), TEXT("AMF"), Result);
		}
	}

	return EAVResult::Success;
}

template <>
DLLEXPORT FAVResult FVideoDecoderAMF::CopySurface<FVideoResourceVulkan>(TVideoDecoderAMF<FVideoResourceVulkan>& This, TSharedPtr<FVideoResourceVulkan>& OutResource, amf::AMFSurfacePtr const& InSurface)
{
	return EAVResult::FatalUnsupported;
}
