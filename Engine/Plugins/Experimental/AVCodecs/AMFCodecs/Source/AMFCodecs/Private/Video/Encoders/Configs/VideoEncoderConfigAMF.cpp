// Copyright Epic Games, Inc. All Rights Reserved.

#include "Video/Encoders/Configs/VideoEncoderConfigAMF.h"

#include "components/VideoEncoderVCE.h"
#include "components/VideoEncoderHEVC.h"

#include "Video/Encoders/Configs/VideoEncoderConfigH264.h"
#include "Video/Encoders/Configs/VideoEncoderConfigH265.h"

REGISTER_TYPEID(FVideoEncoderConfigAMF);

FName const FVideoEncoderConfigAMF::CodecTypeH264 = WCHAR_TO_TCHAR(AMFVideoEncoderVCE_AVC);
FName const FVideoEncoderConfigAMF::CodecTypeH265 = WCHAR_TO_TCHAR(AMFVideoEncoder_HEVC);

TAVResult<AMF_VIDEO_ENCODER_RATE_CONTROL_METHOD_ENUM> FVideoEncoderConfigAMF::ConvertRateControlMode(ERateControlMode Mode)
{
	switch (Mode)
	{
	case ERateControlMode::ConstQP:
		return AMF_VIDEO_ENCODER_RATE_CONTROL_METHOD_ENUM::AMF_VIDEO_ENCODER_RATE_CONTROL_METHOD_CONSTANT_QP;
	case ERateControlMode::VBR:
		return AMF_VIDEO_ENCODER_RATE_CONTROL_METHOD_ENUM::AMF_VIDEO_ENCODER_RATE_CONTROL_METHOD_QUALITY_VBR; // TODO(Aidan) review?
	case ERateControlMode::CBR:
		return AMF_VIDEO_ENCODER_RATE_CONTROL_METHOD_ENUM::AMF_VIDEO_ENCODER_RATE_CONTROL_METHOD_CBR;
	default:
		return FAVResult(EAVResult::ErrorUnsupported, FString::Printf(TEXT("Rate control mode %d is not supported"), EnumToUnderlyingType(Mode)), TEXT("AMF"));
	}
}

TAVResult<ERateControlMode> FVideoEncoderConfigAMF::ConvertRateControlMode(AMF_VIDEO_ENCODER_RATE_CONTROL_METHOD_ENUM Mode)
{
	switch (Mode)
	{
	case AMF_VIDEO_ENCODER_RATE_CONTROL_METHOD_ENUM::AMF_VIDEO_ENCODER_RATE_CONTROL_METHOD_CONSTANT_QP:
		return ERateControlMode::ConstQP;
	case AMF_VIDEO_ENCODER_RATE_CONTROL_METHOD_ENUM::AMF_VIDEO_ENCODER_RATE_CONTROL_METHOD_QUALITY_VBR: // TODO(Aidan) review?
		return ERateControlMode::VBR;
	case AMF_VIDEO_ENCODER_RATE_CONTROL_METHOD_ENUM::AMF_VIDEO_ENCODER_RATE_CONTROL_METHOD_CBR:
		return ERateControlMode::CBR;
	default:
		return FAVResult(EAVResult::ErrorUnsupported, FString::Printf(TEXT("Rate control mode %d is not supported"), EnumToUnderlyingType(Mode)), TEXT("AMF"));
	}
}

static uint32 DEFAULT_BITRATE_TARGET = 1000000;
static uint32 DEFAULT_BITRATE_MAX = 10000000;

template<>
DLLEXPORT FAVResult FAVExtension::TransformConfig(FVideoEncoderConfigAMF& OutConfig, FVideoEncoderConfig const& InConfig)
{
	OutConfig.Preset = InConfig.Preset;
	OutConfig.LatencyMode = InConfig.LatencyMode;

	OutConfig.Width = InConfig.Width;
	OutConfig.Height = InConfig.Height;

	OutConfig.SetProperty(AMF_VIDEO_ENCODER_USAGE, AMF_VIDEO_ENCODER_USAGE_LOW_LATENCY);

	AMFRate const FrameRate = { InConfig.TargetFramerate, 1 };
	OutConfig.SetProperty(AMF_VIDEO_ENCODER_FRAMERATE, FrameRate);

#if PLATFORM_WINDOWS
	TAVResult<AMF_VIDEO_ENCODER_RATE_CONTROL_METHOD_ENUM> const ConvertedRateControlMode = FVideoEncoderConfigAMF::ConvertRateControlMode(InConfig.RateControlMode);
	if (ConvertedRateControlMode.IsNotSuccess())
	{
		return ConvertedRateControlMode;
	}

	OutConfig.SetProperty(AMF_VIDEO_ENCODER_RATE_CONTROL_METHOD, ConvertedRateControlMode.ReturnValue);

	if (InConfig.RateControlMode == ERateControlMode::CBR)
	{
		OutConfig.SetProperty(AMF_VIDEO_ENCODER_FILLER_DATA_ENABLE, InConfig.bFillData);
	}

	OutConfig.SetProperty(AMF_VIDEO_ENCODER_PEAK_BITRATE, InConfig.MaxBitrate > -1 ? InConfig.MaxBitrate : DEFAULT_BITRATE_MAX);
#endif

	OutConfig.SetProperty(AMF_VIDEO_ENCODER_TARGET_BITRATE, InConfig.TargetBitrate > -1 ? InConfig.TargetBitrate : DEFAULT_BITRATE_TARGET);

	OutConfig.SetProperty(AMF_VIDEO_ENCODER_QUALITY_PRESET, AMF_VIDEO_ENCODER_QUALITY_PRESET_QUALITY);
	OutConfig.SetProperty(AMF_VIDEO_ENCODER_B_PIC_PATTERN, 0);

	OutConfig.SetProperty(AMF_VIDEO_ENCODER_MIN_QP, FMath::Floor<amf_int64>((1.0f - (InConfig.MaxQuality / 100.0f)) * 51.0f));
	OutConfig.SetProperty(AMF_VIDEO_ENCODER_MAX_QP, FMath::Floor<amf_int64>((1.0f - (InConfig.MinQuality / 100.0f)) * 51.0f));

	OutConfig.SetProperty(AMF_VIDEO_ENCODER_QUERY_TIMEOUT, 16);

	// Pixel streaming uses Nvidia convention of using 0xffffffff for infinite p-frames.
	// AMD uses 0. See AMF_VIDEO_ENCODER_IDR_PERIOD in https://github.com/GPUOpen-LibrariesAndSDKs/AMF/blob/master/amf/doc/AMF_Video_Encode_API.md
	uint32 KeyframeInterval = InConfig.KeyframeInterval == 0xffffffff ? 0 : InConfig.KeyframeInterval;
	OutConfig.SetProperty(AMF_VIDEO_ENCODER_IDR_PERIOD, KeyframeInterval);

	return EAVResult::Success;
}

template <>
DLLEXPORT FAVResult FAVExtension::TransformConfig(FVideoEncoderConfig& OutConfig, FVideoEncoderConfigAMF const& InConfig)
{
	OutConfig.Preset = InConfig.Preset;
	OutConfig.LatencyMode = InConfig.LatencyMode;

	OutConfig.Width = InConfig.Width;
	OutConfig.Height = InConfig.Height;

	AMFRate FrameRate;
	InConfig.GetProperty(AMF_VIDEO_ENCODER_FRAMERATE, &FrameRate);

	OutConfig.TargetFramerate = FrameRate.num;

#if PLATFORM_WINDOWS
	int32 RateControlMode;
	InConfig.GetProperty(AMF_VIDEO_ENCODER_RATE_CONTROL_METHOD, &RateControlMode);

	TAVResult<ERateControlMode> const ConvertedRateControlMode = FVideoEncoderConfigAMF::ConvertRateControlMode(static_cast<AMF_VIDEO_ENCODER_RATE_CONTROL_METHOD_ENUM>(RateControlMode));
	if (ConvertedRateControlMode.IsNotSuccess())
	{
		return ConvertedRateControlMode;
	}

	OutConfig.RateControlMode = ConvertedRateControlMode;

	bool FillData;
	InConfig.GetProperty(AMF_VIDEO_ENCODER_FILLER_DATA_ENABLE, &FillData);

	OutConfig.bFillData = FillData;

	InConfig.GetProperty(AMF_VIDEO_ENCODER_PEAK_BITRATE, &OutConfig.MaxBitrate);
#endif

	InConfig.GetProperty(AMF_VIDEO_ENCODER_TARGET_BITRATE, &OutConfig.TargetBitrate);

	amf_int64 OutMinQpValue;
	InConfig.GetProperty(AMF_VIDEO_ENCODER_MIN_QP, &OutMinQpValue);
	OutConfig.MaxQuality = (1.0f - (OutMinQpValue / 51.0f)) * 100.0f;

	amf_int64 OutMaxQpValue;
	InConfig.GetProperty(AMF_VIDEO_ENCODER_MAX_QP, &OutMaxQpValue);
	OutConfig.MinQuality = (1.0f - (OutMaxQpValue / 51.0f)) * 100.0f;

	// Pixel streaming uses Nvidia convention of using 0xffffffff for infinite p-frames.
	// AMD uses 0. See AMF_VIDEO_ENCODER_IDR_PERIOD in https://github.com/GPUOpen-LibrariesAndSDKs/AMF/blob/master/amf/doc/AMF_Video_Encode_API.md
	InConfig.GetProperty(AMF_VIDEO_ENCODER_IDR_PERIOD, &OutConfig.KeyframeInterval);
	OutConfig.KeyframeInterval = OutConfig.KeyframeInterval > 0 ? OutConfig.KeyframeInterval : 0xffffffff;

	return EAVResult::Success;
}

template <>
DLLEXPORT FAVResult FAVExtension::TransformConfig(FVideoEncoderConfigAMF& OutConfig, FVideoEncoderConfigH264 const& InConfig)
{
	// Unused from preset
	/*Config.encodeCodecConfig.h264Config.adaptiveTransformMode = NV_ENC_H264_ADAPTIVE_TRANSFORM_ENABLE;
	Config.encodeCodecConfig.h264Config.entropyCodingMode = NV_ENC_H264_ENTROPY_CODING_MODE_CABAC;
	Config.encodeCodecConfig.h264Config.h264VUIParameters.videoFormat = 5;
	Config.encodeCodecConfig.h264Config.h264VUIParameters.colourPrimaries = 2;
	Config.encodeCodecConfig.h264Config.h264VUIParameters.transferCharacteristics = 2;
	Config.encodeCodecConfig.h264Config.h264VUIParameters.colourMatrix = 2;*/

	static auto const ConvertProfile = [](EH264Profile Profile) -> TAVResult<AMF_VIDEO_ENCODER_PROFILE_ENUM>
		{
			switch (Profile)
			{
			case EH264Profile::Baseline:
				return AMF_VIDEO_ENCODER_PROFILE_ENUM::AMF_VIDEO_ENCODER_PROFILE_BASELINE;
			case EH264Profile::Main:
				return AMF_VIDEO_ENCODER_PROFILE_ENUM::AMF_VIDEO_ENCODER_PROFILE_MAIN;
			case EH264Profile::High:
				return AMF_VIDEO_ENCODER_PROFILE_ENUM::AMF_VIDEO_ENCODER_PROFILE_HIGH;
			case EH264Profile::ConstrainedHigh:
				return AMF_VIDEO_ENCODER_PROFILE_ENUM::AMF_VIDEO_ENCODER_PROFILE_CONSTRAINED_HIGH;
			default:
				return FAVResult(EAVResult::ErrorUnsupported, FString::Printf(TEXT("H264 profile %d is not supported"), EnumToUnderlyingType(Profile)), TEXT("AMF"));
			}
		};

	OutConfig.CodecType = FVideoEncoderConfigAMF::CodecTypeH264;
	OutConfig.RepeatSPSPPS = InConfig.RepeatSPSPPS;

	TAVResult<AMF_VIDEO_ENCODER_PROFILE_ENUM> const ConvertedProfile = ConvertProfile(InConfig.Profile);
	if (ConvertedProfile.IsNotSuccess())
	{
		return ConvertedProfile;
	}

	OutConfig.SetProperty(AMF_VIDEO_ENCODER_PROFILE, ConvertedProfile.ReturnValue);
	OutConfig.SetProperty(AMF_VIDEO_ENCODER_PROFILE_LEVEL, 51);

	return FAVExtension::TransformConfig<FVideoEncoderConfigAMF, FVideoEncoderConfig>(OutConfig, InConfig);
}

// TODO (Eden.Harris) H265/HEVC disabled because of intermittent build failure. See UE-239424
// template <>
// DLLEXPORT FAVResult FAVExtension::TransformConfig(FVideoEncoderConfigAMF& OutConfig, FVideoEncoderConfigH265 const& InConfig)
// {
// 	OutConfig.CodecType = FVideoEncoderConfigAMF::CodecTypeH265;

// 	static auto const ConvertProfile = [](EH265Profile Profile) -> TAVResult<AMF_VIDEO_ENCODER_PROFILE_ENUM>
// 		{
// 			switch (Profile)
// 			{
// 			case EH265Profile::Main:
// 				return AMF_VIDEO_ENCODER_PROFILE_ENUM::AMF_VIDEO_ENCODER_PROFILE_MAIN;
// 			default:
// 				return FAVResult(EAVResult::ErrorUnsupported, FString::Printf(TEXT("H265 profile %d is not supported"), Profile), TEXT("AMF"));
// 			}
// 		};

// 	OutConfig.CodecType = FVideoEncoderConfigAMF::CodecTypeH265;
// 	OutConfig.RepeatSPSPPS = InConfig.RepeatSPSPPS;

// 	TAVResult<AMF_VIDEO_ENCODER_PROFILE_ENUM> const ConvertedProfile = ConvertProfile(InConfig.Profile);
// 	if (ConvertedProfile.IsNotSuccess())
// 	{
// 		return ConvertedProfile;
// 	}

// 	OutConfig.SetProperty(AMF_VIDEO_ENCODER_PROFILE, ConvertedProfile.ReturnValue);
// 	OutConfig.SetProperty(AMF_VIDEO_ENCODER_PROFILE_LEVEL, 51);

// 	return FAVExtension::TransformConfig<FVideoEncoderConfigAMF, FVideoEncoderConfig>(OutConfig, InConfig);
// }
