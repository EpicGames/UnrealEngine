// Copyright Epic Games, Inc. All Rights Reserved.

#include "Video/Encoders/VideoEncoderAMF.h"

#include "Video/Resources/Vulkan/VideoResourceVulkan.h"

#if PLATFORM_WINDOWS
	#include "Video/Resources/D3D/VideoResourceD3D.h"
#endif

REGISTER_TYPEID(amf::AMFSurfacePtr);

#if PLATFORM_WINDOWS

template <>
DLLEXPORT FAVResult FVideoEncoderAMF::SetupContext<FVideoResourceD3D11>(TVideoEncoderAMF<FVideoResourceD3D11>& This)
{
	TSharedPtr<FVideoContextD3D11> const& D3D11Context = This.GetDevice()->GetContext<FVideoContextD3D11>();
	if (D3D11Context.IsValid())
	{
		AMF_RESULT const Result = This.Context->InitDX11(D3D11Context->Device);
		if (Result != AMF_OK)
		{
			return FAVResult(EAVResult::ErrorCreating, TEXT("Failed to initialize the D3D11 encoder context"), TEXT("AMF"), Result);
		}
	}

	return EAVResult::Success;
}

template <>
DLLEXPORT FAVResult FVideoEncoderAMF::MapSurface<FVideoResourceD3D11>(TVideoEncoderAMF<FVideoResourceD3D11>& This, TSharedPtr<amf::AMFSurfacePtr>& OutSurface, TSharedPtr<FVideoResourceD3D11> const& InResource)
{
	amf::AMFSurface* ResourceSurface = nullptr;

	AMF_RESULT const Result = This.Context->CreateSurfaceFromDX11Native(InResource->GetRaw(), &ResourceSurface, nullptr);
	if (Result != AMF_OK)
	{
		return FAVResult(EAVResult::ErrorMapping, TEXT("Failed to create surface from D3D11 resource"), TEXT("AMF"), Result);
	}

	OutSurface = MakeShared<amf::AMFSurfacePtr>(ResourceSurface);

	return EAVResult::Success;
}

template <>
DLLEXPORT FAVResult FVideoEncoderAMF::SetupContext<FVideoResourceD3D12>(TVideoEncoderAMF<FVideoResourceD3D12>& This)
{
	TSharedPtr<FVideoContextD3D12> const& D3D12Context = This.GetDevice()->GetContext<FVideoContextD3D12>();
	if (D3D12Context.IsValid())
	{
		AMF_RESULT const Result = amf::AMFContext2Ptr(This.Context)->InitDX12(D3D12Context->Device);
		if (Result != AMF_OK)
		{
			return FAVResult(EAVResult::ErrorCreating, TEXT("Failed to initialize the D3D12 encoder context"), TEXT("AMF"), Result);
		}
	}

	return EAVResult::Success;
}

template <>
DLLEXPORT FAVResult FVideoEncoderAMF::MapSurface<FVideoResourceD3D12>(TVideoEncoderAMF<FVideoResourceD3D12>& This, TSharedPtr<amf::AMFSurfacePtr>& OutSurface, TSharedPtr<FVideoResourceD3D12> const& InResource)
{
	amf::AMFSurface* ResourceSurface = nullptr;

	AMF_RESULT const Result = amf::AMFContext2Ptr(This.Context)->CreateSurfaceFromDX12Native(InResource->GetResource(), &ResourceSurface, nullptr);
	if (Result != AMF_OK)
	{
		return FAVResult(EAVResult::ErrorMapping, TEXT("Failed to create surface from D3D12 resource"), TEXT("AMF"), Result);
	}

	OutSurface = MakeShared<amf::AMFSurfacePtr>(ResourceSurface);

	return EAVResult::Success;
}

#endif

template <>
DLLEXPORT FAVResult FVideoEncoderAMF::SetupContext<FVideoResourceVulkan>(TVideoEncoderAMF<FVideoResourceVulkan>& This)
{
	TSharedPtr<FVideoContextVulkan> const& VulkanContext = This.GetDevice()->GetContext<FVideoContextVulkan>();
	if (VulkanContext.IsValid())
	{
		amf::AMFVulkanDevice AMFDevice = {};
		AMFDevice.cbSizeof = sizeof(amf::AMFVulkanDevice);
		AMFDevice.hInstance = VulkanContext->Instance;
		AMFDevice.hPhysicalDevice = VulkanContext->PhysicalDevice;
		AMFDevice.hDevice = VulkanContext->Device;

		AMF_RESULT const Result = amf::AMFContext1Ptr(This.Context)->InitVulkan(&AMFDevice);
		if (Result != AMF_OK)
		{
			return FAVResult(EAVResult::ErrorCreating, TEXT("Failed to initialize the vulkan encoder context"), TEXT("AMF"), Result);
		}
	}

	return EAVResult::Success;
}

template <>
DLLEXPORT FAVResult FVideoEncoderAMF::MapSurface<FVideoResourceVulkan>(TVideoEncoderAMF<FVideoResourceVulkan>& This, TSharedPtr<amf::AMFSurfacePtr>& OutSurface, TSharedPtr<FVideoResourceVulkan> const& InResource)
{
	amf::AMFSurface* ResourceSurface = nullptr;

	AMF_RESULT const Result = amf::AMFContext1Ptr(This.Context)->CreateSurfaceFromVulkanNative(static_cast<void*>(InResource->GetRaw()), &ResourceSurface, nullptr);
	if (Result != AMF_OK)
	{
		return FAVResult(EAVResult::ErrorMapping, TEXT("Failed to create surface from vulkan resource"), TEXT("AMF"), Result);
	}

	OutSurface = MakeShared<amf::AMFSurfacePtr>(ResourceSurface);

	return EAVResult::Success;
}
