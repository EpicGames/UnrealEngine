// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AVUtility.h"

#include "CoreMinimal.h"
#include "RHIDefinitions.h"
#include "HAL/Thread.h"

#include <string.h>

THIRD_PARTY_INCLUDES_START

#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"
#pragma warning(push)
#pragma warning(disable : 4005)
#endif //PLATFORM_WINDOWS

#include "core/Factory.h"
#include "core/Interface.h"
#include "core/VulkanAMF.h"
#include "components/VideoEncoderVCE.h"

#if PLATFORM_WINDOWS
#pragma warning(pop)
#include "Windows/HideWindowsPlatformTypes.h"
#endif //PLATFORM_WINDOWS

THIRD_PARTY_INCLUDES_END

class FAMF : public FAPI
{
public:
	bool bHasCompatibleGPU = true;

	AMFCODECS_API FAMF();

	AMFCODECS_API virtual bool IsValid() const override;

	amf::AMFFactory* GetFactory() const { return Factory; }

private:
	amf_handle DllHandle = nullptr;
	amf::AMFFactory* Factory = nullptr;
};

DECLARE_TYPEID(FAMF, AMFCODECS_API);
