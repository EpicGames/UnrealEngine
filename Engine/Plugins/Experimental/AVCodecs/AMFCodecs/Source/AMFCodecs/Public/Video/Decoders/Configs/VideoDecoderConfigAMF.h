// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AVExtension.h"
#include "Video/VideoDecoder.h"

#include "AMF.h"

#define UE_API AMFCODECS_API

struct FVideoDecoderConfigAMF : public FAVConfig
{
private:
	TMap<wchar_t const*, amf::AMFVariantStruct, FDefaultSetAllocator, TStringPointerMapKeyFuncs_DEPRECATED<wchar_t const*, amf::AMFVariantStruct>> Variants;

public:
	struct FParsedPicture
	{
	public:
		TArray<uint8> ExtraData;
	};

	static UE_API FName const CodecTypeH264;
	static UE_API FName const CodecTypeH265;

	FName CodecType;

	uint32 Width = 1920;
	uint32 Height = 1080;

	FVideoDecoderConfigAMF(EAVPreset Preset = EAVPreset::Default)
		: FAVConfig(Preset)
	{
	}

	bool operator==(FVideoDecoderConfigAMF const& Other) const
	{
		if (CodecType == Other.CodecType && Width == Other.Width && Height == Other.Height)
		{
			if (Variants.Num() != Other.Variants.Num())
			{
				return false;
			}

			for (TPair<wchar_t const*, amf::AMFVariantStruct> const& Variant : Variants)
			{
				amf::AMFVariantStruct const* OtherVariantValue = Other.Variants.Find(Variant.Key);
				if (OtherVariantValue == nullptr)
				{
					return false;
				}

				bool IsEqual = false;
				AMFVariantCompare(&Variant.Value, OtherVariantValue, &IsEqual);

				if (!IsEqual)
				{
					return false;
				}
			}
		}

		return false;
	}

	bool operator!=(FVideoDecoderConfigAMF const& Other) const
	{
		return !(*this == Other);
	}

	template <typename TVariant>
	void SetProperty(wchar_t const* Name, TVariant const& Variant)
	{
		Variants.Add(Name, static_cast<const amf::AMFVariantStruct&>(amf::AMFVariant(Variant)));
	}

	template <typename TVariant>
	bool GetProperty(wchar_t const* Name, TVariant* OutVariant) const
	{
		if (amf::AMFVariant const* const Variant = static_cast<amf::AMFVariant const*>(Variants.Find(Name)))
		{
			*OutVariant = static_cast<TVariant>(*Variant);

			return true;
		}

		return false;
	}

	void CopyTo(amf::AMFPropertyStorage* Other) const
	{
		for (TPair<wchar_t const*, amf::AMFVariantStruct> const& Variant : Variants)
		{
			Other->SetProperty(Variant.Key, Variant.Value);
		}
	}

	void CopyFrom(amf::AMFPropertyStorage const* Other)
	{
		int32 const VariantCount = Other->GetPropertyCount();
		for (int i = 0; i < VariantCount; ++i)
		{
			wchar_t VariantName[128];
			amf::AMFVariantStruct VariantValue;
			if (Other->GetPropertyAt(i, VariantName, sizeof(VariantName), &VariantValue) == AMF_OK)
			{
				Variants.Add(VariantName, VariantValue);
			}
		}
	}

	UE_API FAVResult Parse(TSharedRef<FAVInstance> const& Instance, FVideoPacket const& Packet, TArray<FParsedPicture>& OutPictures);
};

template <>
FAVResult FAVExtension::TransformConfig(FVideoDecoderConfigAMF& OutConfig, struct FVideoDecoderConfig const& InConfig);

template <>
FAVResult FAVExtension::TransformConfig(struct FVideoDecoderConfig& OutConfig, FVideoDecoderConfigAMF const& InConfig);

template <>
FAVResult FAVExtension::TransformConfig(FVideoDecoderConfigAMF& OutConfig, struct FVideoDecoderConfigH264 const& InConfig);

template <>
FAVResult FAVExtension::TransformConfig(FVideoDecoderConfigAMF& OutConfig, struct FVideoDecoderConfigH265 const& InConfig);

DECLARE_TYPEID(FVideoDecoderConfigAMF, AMFCODECS_API);

#undef UE_API
