// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Video/VideoDecoder.h"
#include "Video/Decoders/Configs/VideoDecoderConfigAMF.h"

#include "Containers/Queue.h"

#include "AMF.h"

template <typename TResource>
class TVideoDecoderAMF : public TVideoDecoder<TResource, FVideoDecoderConfigAMF>
{
private:
	amf::AMFComponentPtr Decoder = nullptr;

	struct FFrame
	{
		int32 SurfaceIndex;
		uint32 Width;
		uint32 Height;
	};

	TQueue<FFrame> Frames;

public:
	amf::AMFContextPtr Context = nullptr;
	
	TVideoDecoderAMF() = default;
	virtual ~TVideoDecoderAMF() override;

	virtual bool IsOpen() const override;
	virtual FAVResult Open(TSharedRef<FAVDevice> const& NewDevice, TSharedRef<FAVInstance> const& NewInstance) override;
	virtual void Close() override;

	bool IsInitialized() const;

	virtual FAVResult ApplyConfig() override;

	//int GetCapability(GUID EncodeGUID, NV_ENC_CAPS CapsToQuery) const;

	virtual FAVResult SendPacket(FVideoPacket const& Packet) override;
	
	virtual FAVResult ReceiveFrame(TResolvableVideoResource<TResource>& InOutResource) override;
};

// TODO (Andrew) Hack until I get back to AMF
struct FVideoDecoderAMF
{
	template <typename TResource>
	static FAVResult SetupContext(TVideoDecoderAMF<TResource>& This) = delete;

	template <typename TResource>
	static FAVResult CopySurface(TVideoDecoderAMF<TResource>& This, TSharedPtr<TResource>& OutResource, amf::AMFSurfacePtr const& InSurface) = delete;
};

#include "VideoDecoderAMF.hpp"

#if PLATFORM_WINDOWS

template <>
FAVResult FVideoDecoderAMF::SetupContext<class FVideoResourceD3D11>(TVideoDecoderAMF<class FVideoResourceD3D11>& This);

template <>
FAVResult FVideoDecoderAMF::CopySurface<class FVideoResourceD3D11>(TVideoDecoderAMF<FVideoResourceD3D11>& This, TSharedPtr<FVideoResourceD3D11>& OutResource, amf::AMFSurfacePtr const& InSurface);

template <>
FAVResult FVideoDecoderAMF::SetupContext<class FVideoResourceD3D12>(TVideoDecoderAMF<class FVideoResourceD3D12>& This);

template <>
FAVResult FVideoDecoderAMF::CopySurface<class FVideoResourceD3D12>(TVideoDecoderAMF<FVideoResourceD3D12>& This, TSharedPtr<FVideoResourceD3D12>& OutResource, amf::AMFSurfacePtr const& InSurface);

#endif

template <>
FAVResult FVideoDecoderAMF::SetupContext<class FVideoResourceVulkan>(TVideoDecoderAMF<class FVideoResourceVulkan>& This);

template <>
FAVResult FVideoDecoderAMF::CopySurface<class FVideoResourceVulkan>(TVideoDecoderAMF<FVideoResourceVulkan>& This, TSharedPtr<FVideoResourceVulkan>& OutResource, amf::AMFSurfacePtr const& InSurface);
