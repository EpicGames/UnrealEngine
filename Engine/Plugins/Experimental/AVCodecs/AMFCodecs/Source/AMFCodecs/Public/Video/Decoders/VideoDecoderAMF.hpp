// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "components/VideoDecoderUVD.h"

template <typename TResource>
TVideoDecoderAMF<TResource>::~TVideoDecoderAMF()
{
	Close();
}

template <typename TResource>
bool TVideoDecoderAMF<TResource>::IsOpen() const
{
	return Context != nullptr;
}

template <typename TResource>
FAVResult TVideoDecoderAMF<TResource>::Open(TSharedRef<FAVDevice> const& NewDevice, TSharedRef<FAVInstance> const& NewInstance)
{
	Close();
	
	TVideoDecoder<TResource, FVideoDecoderConfigAMF>::Open(NewDevice, NewInstance);

	AMF_RESULT const Result = FAPI::Get<FAMF>().GetFactory()->CreateContext(&Context);
	if (Result != AMF_OK)
	{
		return FAVResult(EAVResult::ErrorCreating, TEXT("Failed to create decoder context"), TEXT("AMF"), Result);
	}

	FAVResult const AVResult = FVideoDecoderAMF::template SetupContext<TResource>(*this);
	if (AVResult.IsNotSuccess())
	{
		return AVResult;
	}

	return EAVResult::Success;
}
	
template <typename TResource>
void TVideoDecoderAMF<TResource>::Close()
{
	if (IsOpen())
	{
		if (Decoder != nullptr)
		{
			AMF_RESULT const Result = Decoder->Terminate();
			if (Result != AMF_OK)
			{
				FAVResult::Log(EAVResult::ErrorDestroying, TEXT("Failed to destroy decoder component"), TEXT("AMF"), Result);
			}

			Decoder = nullptr;
		}

		if (Context != nullptr)
		{
			AMF_RESULT const Result = Context->Terminate();
			if (Result != AMF_OK)
			{
				FAVResult::Log(EAVResult::ErrorDestroying, TEXT("Failed to destroy decoder context"), TEXT("AMF"), Result);
			}

			Context = nullptr;
		}
	}
}

template <typename TResource>
bool TVideoDecoderAMF<TResource>::IsInitialized() const
{
	return Decoder != nullptr;
}

template <typename TResource>
FAVResult TVideoDecoderAMF<TResource>::ApplyConfig()
{
	if (IsOpen())
	{
		FVideoDecoderConfigAMF const& PendingConfig = this->GetPendingConfig();
		if (this->AppliedConfig != PendingConfig)
		{
			if (IsInitialized())
			{
				PendingConfig.CopyTo(Decoder);
			}

			if (!IsInitialized())
			{
				AMF_RESULT Result = FAPI::Get<FAMF>().GetFactory()->CreateComponent(Context, TCHAR_TO_WCHAR(*PendingConfig.CodecType.ToString()), &Decoder);
				if (Result != AMF_OK)
				{
					return FAVResult(EAVResult::ErrorCreating, TEXT("Failed to create decoder component"), TEXT("AMF"), Result);
				}

				PendingConfig.CopyTo(Decoder);

				Result = Decoder->Init(amf::AMF_SURFACE_BGRA, PendingConfig.Width, PendingConfig.Height);
				if (Result != AMF_OK)
				{
					Close();

					return FAVResult(EAVResult::ErrorCreating, TEXT("Failed to initialize decoder"), TEXT("AMF"), Result);
				}
			}
		}

		return TVideoDecoder<TResource, FVideoDecoderConfigAMF>::ApplyConfig();
	}

	return FAVResult(EAVResult::ErrorInvalidState, TEXT("Decoder not open"), TEXT("AMF"));
}

/*template <typename TResource>
int TVideoDecoderAMF<TResource>::GetCapability(GUID EncodeGUID, NV_ENC_CAPS CapsToQuery) const
{
	if (IsOpen())
	{
		int CapsValue = 0;

		NV_ENC_STRUCT(NV_ENC_CAPS_PARAM, CapsParam);
		CapsParam.capsToQuery = CapsToQuery;

		AMFSTATUS const Result = FAPI::Get<FAMF>().AMFGetEncodeCaps(Decoder, EncodeGUID, &CapsParam, &CapsValue);
		if (Result != NV_ENC_SUCCESS)
		{
			FAVResult::Log(EAVResult::Warning, FString::Printf(TEXT("Failed to query for AMF capability %d"), CapsToQuery), TEXT("AMF"), Result);

			return 0;
		}

		return CapsValue;
	}

	return FAVResult(EAVResult::ErrorInvalidState, TEXT("Decoder not open"), TEXT("AMF"));
}*/

template <typename TResource>
FAVResult TVideoDecoderAMF<TResource>::SendPacket(FVideoPacket const& Packet)
{
	if (IsOpen())
	{
		FVideoDecoderConfigAMF& PendingConfig = this->EditPendingConfig();

		TArray<FVideoDecoderConfigAMF::FParsedPicture> Pictures;

		FAVResult const AVResult = PendingConfig.Parse(this->GetInstance().ToSharedRef(), Packet, Pictures);
		if (AVResult.IsNotSuccess())
		{
			return AVResult;
		}

		for (auto& Picture : Pictures)
		{
			amf::AMFBufferPtr PacketBuffer;
			Context->AllocBuffer(amf::AMF_MEMORY_HOST, Picture.ExtraData.Num(), &PacketBuffer);

			FMemory::Memcpy(PacketBuffer->GetNative(), Picture.ExtraData.GetData(), Picture.ExtraData.Num());
			Decoder->SetProperty(AMF_VIDEO_DECODER_EXTRADATA, amf::AMFVariant(PacketBuffer));

			ApplyConfig();

			// TODO (Andrew) Get rid of this shitty copy, use an AMFDataPtr somehow
			amf::AMFBufferPtr PacketData;
			Context->AllocBuffer(amf::AMF_MEMORY_HOST, Packet.DataSize, &PacketData);

			FMemory::Memcpy(PacketData->GetNative(), Packet.DataPtr.Get(), Packet.DataSize);

			AMF_RESULT const Result = Decoder->SubmitInput(PacketData);
			if (Result != AMF_OK)
			{
				if (Result == AMF_NEED_MORE_INPUT)
				{
					return EAVResult::PendingInput;
				}

				if (Result == AMF_INPUT_FULL || Result == AMF_DECODER_NO_FREE_SURFACES)
				{
					return EAVResult::PendingOutput;
				}

				return FAVResult(EAVResult::Error, TEXT("Failed to decode frame"), TEXT("AMF"), Result);
			}
		}

		return EAVResult::Success;
	}

	return FAVResult(EAVResult::ErrorInvalidState, TEXT("Decoder not open"), TEXT("AMF"));
}

template <typename TResource>
FAVResult TVideoDecoderAMF<TResource>::ReceiveFrame(TResolvableVideoResource<TResource>& InOutResource)
{
	if (IsOpen())
	{
		if (IsInitialized())
		{
			amf::AMFDataPtr FrameData;
			AMF_RESULT const Result = Decoder->QueryOutput(&FrameData);
			if (Result != AMF_OK)
			{
				if (Result == AMF_NEED_MORE_INPUT)
				{
					return EAVResult::PendingInput;
				}

				return FAVResult(EAVResult::Error, TEXT("Failed to decode frame"), TEXT("AMF"), Result);
			}

			if (FrameData == nullptr)
			{
				return EAVResult::PendingInput;
			}

			amf::AMFSurfacePtr const FrameSurface(FrameData);
			amf::AMFPlanePtr const FramePlane = FrameSurface->GetPlaneAt(0);

			if (!InOutResource.Resolve(this->GetDevice(), FVideoDescriptor(EVideoFormat::NV12, FramePlane->GetWidth(), FramePlane->GetHeight())))
			{
				return FAVResult(EAVResult::ErrorResolving, TEXT("Failed to resolve frame resource"), TEXT("AMF"));
			}

			FScopeLock const ResourceLock = InOutResource->LockScope();

			return FVideoDecoderAMF::CopySurface<TResource>(*this, InOutResource, FrameSurface);
		}

		return EAVResult::PendingInput;
	}

	return FAVResult(EAVResult::ErrorInvalidState, TEXT("Decoder not open"), TEXT("AMF"));
}
