// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Video/VideoEncoder.h"
#include "Video/Encoders/Configs/VideoEncoderConfigAMF.h"

#include "Containers/Queue.h"
#include "HAL/Platform.h"

#include "AMF.h"

template <typename TResource>
class TVideoEncoderAMF : public TVideoEncoder<TResource, FVideoEncoderConfigAMF>
{
private:
	amf::AMFComponentPtr Encoder = nullptr;
	uint8 bInitialized : 1;

	uint64 FrameCount = 0;

	TQueue<FVideoPacket> Packets;

public:
	amf::AMFContextPtr Context = nullptr;
	
	TVideoEncoderAMF() = default;
	virtual ~TVideoEncoderAMF() override;

	virtual bool IsOpen() const override;
	virtual FAVResult Open(TSharedRef<FAVDevice> const& NewDevice, TSharedRef<FAVInstance> const& NewInstance) override;
	virtual void Close() override;

	bool IsInitialized() const;

	virtual FAVResult ApplyConfig() override;

	//int GetCapability(GUID EncodeGUID, NV_ENC_CAPS CapsToQuery) const;
	
	virtual FAVResult SendFrame(TSharedPtr<TResource> const& Resource, uint32 Timestamp, bool bForceKeyframe = false) override;
	FAVResult SendFrame(amf::AMFSurfacePtr Input, bool bShouldApplyConfig = true);

	virtual FAVResult ReceivePacket(FVideoPacket& OutPacket) override;
};

// TODO (Andrew) Hack until I get back to AMF
struct FVideoEncoderAMF
{
	template <typename TResource>
	static FAVResult SetupContext(TVideoEncoderAMF<TResource>& This) = delete;

	template <typename TResource>
	static FAVResult MapSurface(TVideoEncoderAMF<TResource>& This, TSharedPtr<amf::AMFSurfacePtr>& OutSurface, TSharedPtr<TResource> const& InResource) = delete;
};

#include "VideoEncoderAMF.hpp"

#if PLATFORM_WINDOWS

template <>
FAVResult FVideoEncoderAMF::SetupContext<class FVideoResourceD3D11>(TVideoEncoderAMF<class FVideoResourceD3D11>& This);

template <>
FAVResult FVideoEncoderAMF::MapSurface<class FVideoResourceD3D11>(TVideoEncoderAMF<class FVideoResourceD3D11>& This, TSharedPtr<amf::AMFSurfacePtr>& OutSurface, TSharedPtr<class FVideoResourceD3D11> const& InResource);

template <>
FAVResult FVideoEncoderAMF::SetupContext<class FVideoResourceD3D12>(TVideoEncoderAMF<class FVideoResourceD3D12>& This);

template <>
FAVResult FVideoEncoderAMF::MapSurface<class FVideoResourceD3D12>(TVideoEncoderAMF<class FVideoResourceD3D12>& This, TSharedPtr<amf::AMFSurfacePtr>& OutSurface, TSharedPtr<class FVideoResourceD3D12> const& InResource);

#endif

template <>
FAVResult FVideoEncoderAMF::SetupContext<class FVideoResourceVulkan>(TVideoEncoderAMF<class FVideoResourceVulkan>& This);

template <>
FAVResult FVideoEncoderAMF::MapSurface<class FVideoResourceVulkan>(TVideoEncoderAMF<class FVideoResourceVulkan>& This, TSharedPtr<amf::AMFSurfacePtr>& OutSurface, TSharedPtr<class FVideoResourceVulkan> const& InResource);

DECLARE_TYPEID(amf::AMFSurfacePtr, AMFCODECS_API);
