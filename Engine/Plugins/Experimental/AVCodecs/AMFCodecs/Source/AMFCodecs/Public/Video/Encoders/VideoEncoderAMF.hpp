// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

template <typename TResource>
TVideoEncoderAMF<TResource>::~TVideoEncoderAMF()
{
	Close();
}

template <typename TResource>
bool TVideoEncoderAMF<TResource>::IsOpen() const
{
	return Context != nullptr;
}

template <typename TResource>
FAVResult TVideoEncoderAMF<TResource>::Open(TSharedRef<FAVDevice> const& NewDevice, TSharedRef<FAVInstance> const& NewInstance)
{
	Close();

	TVideoEncoder<TResource, FVideoEncoderConfigAMF>::Open(NewDevice, NewInstance);

	FrameCount = 0;

	AMF_RESULT const Result = FAPI::Get<FAMF>().GetFactory()->CreateContext(&Context);
	if (Result != AMF_OK)
	{
		return FAVResult(EAVResult::ErrorCreating, TEXT("Failed to create encoder context"), TEXT("AMF"), Result);
	}

	FAVResult const AVResult = FVideoEncoderAMF::template SetupContext<TResource>(*this);
	if (AVResult.IsNotSuccess())
	{
		return AVResult;
	}

	return EAVResult::Success;
}

template <typename TResource>
void TVideoEncoderAMF<TResource>::Close()
{
	if (IsOpen())
	{
		bInitialized = false;

		if (Encoder != nullptr)
		{
			AMF_RESULT const Result = Encoder->Terminate();
			if (Result != AMF_OK)
			{
				FAVResult::Log(EAVResult::ErrorDestroying, TEXT("Failed to destroy encoder component"), TEXT("AMF"), Result);
			}

			Encoder = nullptr;
		}

		if (Context != nullptr)
		{
			AMF_RESULT const Result = Context->Terminate();
			if (Result != AMF_OK)
			{
				FAVResult::Log(EAVResult::ErrorDestroying, TEXT("Failed to destroy encoder context"), TEXT("AMF"), Result);
			}

			Context = nullptr;
		}
	}
}

template <typename TResource>
bool TVideoEncoderAMF<TResource>::IsInitialized() const
{
	return Encoder != nullptr;
}

template <typename TResource>
FAVResult TVideoEncoderAMF<TResource>::ApplyConfig()
{
	if (IsOpen())
	{
		FVideoEncoderConfigAMF const& PendingConfig = this->GetPendingConfig();
		if (this->AppliedConfig != PendingConfig)
		{
			if (IsInitialized())
			{
				// Don't have unreconfigurable changes? See VideoEncoderVCE.h
				if (this->AppliedConfig.CompareProperty(AMF_VIDEO_ENCODER_FRAMESIZE, PendingConfig) && this->AppliedConfig.CompareProperty(AMF_VIDEO_ENCODER_FRAMERATE, PendingConfig) && this->AppliedConfig.CompareProperty(AMF_VIDEO_ENCODER_EXTRADATA, PendingConfig) && this->AppliedConfig.CompareProperty(AMF_VIDEO_ENCODER_USAGE, PendingConfig) && this->AppliedConfig.CompareProperty(AMF_VIDEO_ENCODER_PROFILE, PendingConfig) && this->AppliedConfig.CompareProperty(AMF_VIDEO_ENCODER_PROFILE_LEVEL, PendingConfig) && this->AppliedConfig.CompareProperty(AMF_VIDEO_ENCODER_MAX_LTR_FRAMES, PendingConfig) && this->AppliedConfig.CompareProperty(AMF_VIDEO_ENCODER_SCANTYPE, PendingConfig) && this->AppliedConfig.CompareProperty(AMF_VIDEO_ENCODER_MAX_NUM_REFRAMES, PendingConfig) && this->AppliedConfig.CompareProperty(AMF_VIDEO_ENCODER_ASPECT_RATIO, PendingConfig) && this->AppliedConfig.CompareProperty(AMF_VIDEO_ENCODER_FULL_RANGE_COLOR, PendingConfig) && this->AppliedConfig.CompareProperty(AMF_VIDEO_ENCODER_LOWLATENCY_MODE, PendingConfig) && this->AppliedConfig.CompareProperty(AMF_VIDEO_ENCODER_PRE_ANALYSIS_ENABLE, PendingConfig) && this->AppliedConfig.CompareProperty(AMF_VIDEO_ENCODER_PREENCODE_ENABLE, PendingConfig) && this->AppliedConfig.CompareProperty(AMF_VIDEO_ENCODER_QUALITY_PRESET, PendingConfig) && this->AppliedConfig.CompareProperty(AMF_VIDEO_ENCODER_COLOR_BIT_DEPTH, PendingConfig) && this->AppliedConfig.CompareProperty(AMF_VIDEO_ENCODER_INPUT_COLOR_PROFILE, PendingConfig) && this->AppliedConfig.CompareProperty(AMF_VIDEO_ENCODER_INPUT_TRANSFER_CHARACTERISTIC, PendingConfig) && this->AppliedConfig.CompareProperty(AMF_VIDEO_ENCODER_INPUT_COLOR_PRIMARIES, PendingConfig) && this->AppliedConfig.CompareProperty(AMF_VIDEO_ENCODER_INPUT_HDR_METADATA, PendingConfig) && this->AppliedConfig.CompareProperty(AMF_VIDEO_ENCODER_OUTPUT_COLOR_PROFILE, PendingConfig) && this->AppliedConfig.CompareProperty(AMF_VIDEO_ENCODER_OUTPUT_TRANSFER_CHARACTERISTIC, PendingConfig) && this->AppliedConfig.CompareProperty(AMF_VIDEO_ENCODER_OUTPUT_COLOR_PRIMARIES, PendingConfig) && this->AppliedConfig.CompareProperty(AMF_VIDEO_ENCODER_OUTPUT_HDR_METADATA, PendingConfig) &&
					// If the height or width is different, we need to reconstruct the encoder
					this->AppliedConfig.Width == PendingConfig.Width && this->AppliedConfig.Height == PendingConfig.Height)
				{
					this->AppliedConfig.SetDifferencesOnly(PendingConfig, Encoder);
				}
				else
				{
					AMF_RESULT const Result = Encoder->Terminate();
					if (Result != AMF_OK)
					{
						FAVResult::Log(EAVResult::ErrorDestroying, TEXT("Failed to destroy encoder component"), TEXT("AMF"), Result);
					}

					Encoder = nullptr;
				}
			}

			if (!IsInitialized())
			{
				AMF_RESULT Result = FAPI::Get<FAMF>().GetFactory()->CreateComponent(Context, TCHAR_TO_WCHAR(*PendingConfig.CodecType.ToString()), &Encoder);
				if (Result != AMF_OK)
				{
					return FAVResult(EAVResult::ErrorCreating, TEXT("Failed to create encoder component"), TEXT("AMF"), Result);
				}

				PendingConfig.CopyTo(Encoder);

				Result = Encoder->Init(amf::AMF_SURFACE_BGRA, PendingConfig.Width, PendingConfig.Height);
				if (Result != AMF_OK)
				{
					Close();

					return FAVResult(EAVResult::ErrorCreating, TEXT("Failed to initialize encoder"), TEXT("AMF"), Result);
				}
			}
		}

		return TVideoEncoder<TResource, FVideoEncoderConfigAMF>::ApplyConfig();
	}

	return FAVResult(EAVResult::ErrorInvalidState, TEXT("Encoder not open"), TEXT("AMF"));
}

/*template <typename TResource>
int TVideoEncoderAMF<TResource>::GetCapability(GUID EncodeGUID, NV_ENC_CAPS CapsToQuery) const
{
	if (IsOpen())
	{
		int CapsValue = 0;

		NV_ENC_STRUCT(NV_ENC_CAPS_PARAM, CapsParam);
		CapsParam.capsToQuery = CapsToQuery;

		AMFSTATUS const Result = FAPI::Get<FAMF>().AMFGetEncodeCaps(Encoder, EncodeGUID, &CapsParam, &CapsValue);
		if (Result != NV_ENC_SUCCESS)
		{
			FAVResult::Log(EAVResult::Warning, FString::Printf(TEXT("Failed to query for AMF capability %d"), CapsToQuery), TEXT("AMF"), Result);

			return 0;
		}

		return CapsValue;
	}

	return FAVResult(EAVResult::ErrorInvalidState, TEXT("Encoder not open"), TEXT("AMF"));
}*/

template <typename TResource>
FAVResult TVideoEncoderAMF<TResource>::SendFrame(TSharedPtr<TResource> const& Resource, uint32 Timestamp, bool bForceKeyframe)
{
	if (IsOpen())
	{
		FAVResult AVResult = ApplyConfig();
		if (AVResult.IsNotSuccess())
		{
			return AVResult;
		}

		if (Resource.IsValid())
		{
			TSharedPtr<amf::AMFSurfacePtr>& ResourceMapping = Resource->template PinMapping<amf::AMFSurfacePtr>();
			if (!ResourceMapping.IsValid())
			{
				AVResult = FVideoEncoderAMF::template MapSurface<TResource>(*this, ResourceMapping, Resource);
				if (AVResult.IsNotSuccess())
				{
					return AVResult;
				}
			}

			ResourceMapping->GetPtr()->SetPts(Timestamp);

#if PLATFORM_WINDOWS
			ResourceMapping->GetPtr()->SetProperty(AMF_VIDEO_ENCODER_STATISTICS_FEEDBACK, true);
#endif

			if (bForceKeyframe)
			{
				AMF_RESULT Result = ResourceMapping->GetPtr()->SetProperty(AMF_VIDEO_ENCODER_FORCE_PICTURE_TYPE, AMF_VIDEO_ENCODER_PICTURE_TYPE_IDR);
				if (Result != AMF_OK)
				{
					return FAVResult(EAVResult::ErrorMapping, TEXT("Failed to force keyframe"), TEXT("AMF"), Result);
				}

				if (this->AppliedConfig.RepeatSPSPPS)
				{
					Result = ResourceMapping->GetPtr()->SetProperty(AMF_VIDEO_ENCODER_INSERT_SPS, true);
					if (Result != AMF_OK)
					{
						return FAVResult(EAVResult::ErrorMapping, TEXT("Failed to force SPS"), TEXT("AMF"), Result);
					}

					Result = ResourceMapping->GetPtr()->SetProperty(AMF_VIDEO_ENCODER_INSERT_PPS, true);
					if (Result != AMF_OK)
					{
						return FAVResult(EAVResult::ErrorMapping, TEXT("Failed to force PPS"), TEXT("AMF"), Result);
					}
				}
			}

			FScopeLock const ResourceLock = Resource->LockScope();

			return SendFrame(ResourceMapping->GetPtr(), false);
		}

		return SendFrame(nullptr, false);
	}

	return FAVResult(EAVResult::ErrorInvalidState, TEXT("Encoder not open"), TEXT("AMF"));
}

template <typename TResource>
FAVResult TVideoEncoderAMF<TResource>::SendFrame(amf::AMFSurfacePtr Input, bool bShouldApplyConfig)
{
	if (IsOpen())
	{
		if (bShouldApplyConfig)
		{
			FAVResult AVResult = ApplyConfig();
			if (AVResult.IsNotSuccess())
			{
				return AVResult;
			}
		}

		if (Input != nullptr)
		{
			AMF_RESULT const Result = Encoder->SubmitInput(Input);
			if (Result != AMF_OK)
			{
				return FAVResult(EAVResult::Error, TEXT("Failed to encode frame"), TEXT("AMF"), Result);
			}
			else
			{
				if (Result == AMF_NEED_MORE_INPUT)
				{
					return EAVResult::PendingInput;
				}

				amf::AMFDataPtr PacketData;
				while (Encoder->QueryOutput(&PacketData) != AMF_OK)
				{
				} // this seems like a bad test

				amf::AMFBufferPtr const PacketBuffer(PacketData);

				TSharedPtr<uint8> const CopiedData = MakeShareable(new uint8[PacketBuffer->GetSize()]);
				FMemory::BigBlockMemcpy(CopiedData.Get(), PacketBuffer->GetNative(), PacketBuffer->GetSize());

				int32 PacketQP = 0;
				uint32 PacketType = 0;

				// Will fail on Vulkan, oh well
				PacketData->GetProperty(AMF_VIDEO_ENCODER_STATISTIC_FRAME_QP, &PacketQP);

				PacketData->GetProperty(AMF_VIDEO_ENCODER_OUTPUT_DATA_TYPE, &PacketType);

				Packets.Enqueue(
					FVideoPacket(
						CopiedData,
						PacketBuffer->GetSize(),
						PacketBuffer->GetPts(),
						++FrameCount,
						PacketQP,
						PacketType == AMF_VIDEO_ENCODER_OUTPUT_DATA_TYPE_IDR));
			}
		}
		else
		{
			Encoder->Flush();
		}

		return EAVResult::Success;
	}

	return FAVResult(EAVResult::ErrorInvalidState, TEXT("Encoder not open"), TEXT("AMF"));
}

template <typename TResource>
FAVResult TVideoEncoderAMF<TResource>::ReceivePacket(FVideoPacket& OutPacket)
{
	if (IsOpen())
	{
		if (Packets.Dequeue(OutPacket))
		{
			return EAVResult::Success;
		}

		return EAVResult::PendingInput;
	}

	return FAVResult(EAVResult::ErrorInvalidState, TEXT("Encoder not open"), TEXT("AMF"));
}
