// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;
using System;

public class AMFCodecsRHI : ModuleRules
{
	public AMFCodecsRHI(ReadOnlyTargetRules Target) : base(Target)
	{
		bLegacyPublicIncludePaths = false;
		DefaultBuildSettings = BuildSettingsVersion.Latest;
		
		PrivateDependencyModuleNames.AddRange(new string[] {
			"Engine",
			"AVCodecsCore",
			"AMFCodecs",
			"Amf",
			"RHI",
			"VulkanRHI"
		});

		PublicDependencyModuleNames.AddRange(new string[] {
			"RenderCore",
			"Core",
			"CoreUObject",
		});
		
		PrivateDependencyModuleNames.Add("Vulkan");
		AddEngineThirdPartyPrivateStaticDependencies(Target, "Vulkan");

		if (Target.IsInPlatformGroup(UnrealPlatformGroup.Windows))
		{
			PrivateDependencyModuleNames.AddRange(new string[]
			{
				"D3D11RHI",
				"D3D12RHI",
			});
			
			AddEngineThirdPartyPrivateStaticDependencies(Target, "DX11");
			AddEngineThirdPartyPrivateStaticDependencies(Target, "DX12");
		}
	}
}