// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"

#include "IVulkanDynamicRHI.h"
#include "Misc/App.h"

#if PLATFORM_WINDOWS
#include "ID3D11DynamicRHI.h"
#include "ID3D12DynamicRHI.h"
#endif

#include "AVUtility.h"

#include "AMF.h"

class FAMFCodecRHI : public IModuleInterface
{
public:
	virtual void StartupModule() override
	{
		if (FApp::CanEverRender())
		{
			FCoreDelegates::GetOnPostEngineInit().AddLambda([]()
				{
					const ERHIInterfaceType RHIType = GDynamicRHI ? RHIGetInterfaceType() : ERHIInterfaceType::Hidden;
					if (RHIType == ERHIInterfaceType::D3D12 || RHIType == ERHIInterfaceType::D3D11)
					{
						const_cast<FAMF&>(FAPI::Get<FAMF>()).bHasCompatibleGPU = IsRHIDeviceAMD();
					}
					else
					{
						// Vulkan is not supported with our AMF implementation yet.
						const_cast<FAMF&>(FAPI::Get<FAMF>()).bHasCompatibleGPU = false;
					}
				});

			// if (FAPI::Get<FAMF>().IsValid())
			// {
// #if PLATFORM_WINDOWS
// 				TCHAR const* DynamicRHIModuleName = GetSelectedDynamicRHIModuleName(false);
// #elif PLATFORM_LINUX
// 				TCHAR const* DynamicRHIModuleName = TEXT("VulkanRHI");
// #endif

				// TODO (Aidan) AMF is not compatible with Vulkan right now
				/*if(FString("VulkanRHI") == FString(DynamicRHIModuleName))
				{
					AMF.InitializeContext(ERHIInterfaceType::Vulkan, "Vulkan", NULL);
					amf::AMFContext1Ptr pContext1(AMF.GetContext());

					amf_size NumDeviceExtensions = 0;
					pContext1->GetVulkanDeviceExtensions(&NumDeviceExtensions, nullptr);

					TArray<const ANSICHAR*> ExtentionsToAdd; 
					ExtentionsToAdd.AddUninitialized(5);

					pContext1->GetVulkanDeviceExtensions(&NumDeviceExtensions, ExtentionsToAdd.GetData());

					AMF.DestroyContext();

					IVulkanDynamicRHI::AddEnabledDeviceExtensionsAndLayers(ExtentionsToAdd, TArray<const ANSICHAR*>());
				}*/
			// }
		}
	}
};

IMPLEMENT_MODULE(FAMFCodecRHI, AMFCodecsRHI);
